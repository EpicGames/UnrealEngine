{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Portable Object File Data Source",
	"Description" : "Data Source plugin providing portable object (PO) file support for the Content Browser",
	"Category" : "Localization",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "PortableObjectFileDataSource",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ContentBrowserFileDataSource",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		}
	]
}