// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PortableObjectFileDataSource : ModuleRules
	{
		public PortableObjectFileDataSource(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "POFileDataSource";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ContentBrowserFileDataSource",
					"Localization",
				}
			);
		}
	}
}
