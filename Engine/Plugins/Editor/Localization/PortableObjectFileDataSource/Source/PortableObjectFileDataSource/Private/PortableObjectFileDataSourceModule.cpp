// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPortableObjectFileDataSourceModule.h"

#include "ContentBrowserFileDataCore.h"
#include "ContentBrowserFileDataSource.h"
#include "LocalizationDelegates.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#define LOCTEXT_NAMESPACE "PortableObjectFileData"

class FPortableObjectFileDataSourceModule : public IPortableObjectFileDataSourceModule
{
public:
	virtual void StartupModule() override
	{
		if (GIsEditor && !IsRunningCommandlet())
		{
			ContentBrowserFileData::FFileConfigData PoFileConfig;
			{
				static const FText PoReadOnlyError = LOCTEXT("PoReadOnlyError", "Portable Object files are managed by the localization pipeline");

				auto PoCanCreate = [](const FName /*InDestFolderPath*/, const FString& /*InDestFolder*/, FText* OutErrorMsg)
				{
					ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, PoReadOnlyError);
					return false;
				};

				auto PoCanDeleteOrDuplicate = [](const FName /*InFilePath*/, const FString& /*InFilename*/, FText* OutErrorMsg)
				{
					ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, PoReadOnlyError);
					return false;
				};

				auto PoCanRename = [](const FName /*InFilePath*/, const FString& /*InFilename*/, const FString* /*InNewName*/, FText* OutErrorMsg)
				{
					ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, PoReadOnlyError);
					return false;
				};

				auto PoCanCopyOrMove = [](const FName /*InFilePath*/, const FString& /*InFilename*/, const FString& /*InDestFolder*/, FText* OutErrorMsg)
				{
					ContentBrowserFileData::SetOptionalErrorMessage(OutErrorMsg, PoReadOnlyError);
					return false;
				};

				ContentBrowserFileData::FDirectoryActions PoDirectoryActions;
				PoDirectoryActions.CanCreate.BindStatic(PoCanCreate);
				PoDirectoryActions.CanDelete.BindStatic(PoCanDeleteOrDuplicate);
				PoDirectoryActions.CanRename.BindStatic(PoCanRename);
				PoDirectoryActions.CanCopy.BindStatic(PoCanCopyOrMove);
				PoDirectoryActions.CanMove.BindStatic(PoCanCopyOrMove);
				PoDirectoryActions.PassesFilter.BindStatic(&ContentBrowserFileData::FDefaultFileActions::ItemPassesFilter, false);
				PoDirectoryActions.GetAttribute.BindStatic(&ContentBrowserFileData::FDefaultFileActions::GetItemAttribute);
				PoFileConfig.SetDirectoryActions(PoDirectoryActions);

				ContentBrowserFileData::FFileActions PoFileActions;
				PoFileActions.TypeExtension = TEXT("po");
				PoFileActions.TypeName = FTopLevelAssetPath(TEXT("/Script/PortableObject.PortableObject")); // Fake path to satisfy FFileActions requirements
				PoFileActions.TypeDisplayName = LOCTEXT("TypeName", "Portable Object");
				PoFileActions.TypeShortDescription = LOCTEXT("TypeShortDescription", "Portable Object");
				PoFileActions.TypeFullDescription = LOCTEXT("TypeFullDescription", "Portable Object (PO) Translation Data");
				PoFileActions.TypeColor = FColor(200, 191, 231);
				PoFileActions.CanCreate.BindStatic(PoCanCreate);
				PoFileActions.CanDelete.BindStatic(PoCanDeleteOrDuplicate);
				PoFileActions.CanRename.BindStatic(PoCanRename);
				PoFileActions.CanCopy.BindStatic(PoCanCopyOrMove);
				PoFileActions.CanMove.BindStatic(PoCanCopyOrMove);
				PoFileActions.CanDuplicate.BindStatic(PoCanDeleteOrDuplicate);
				PoFileActions.CanEdit.BindRaw(this, &FPortableObjectFileDataSourceModule::CanEditFile);
				PoFileActions.PassesFilter.BindStatic(&ContentBrowserFileData::FDefaultFileActions::ItemPassesFilter, true);
				PoFileActions.GetAttribute.BindStatic(&ContentBrowserFileData::FDefaultFileActions::GetItemAttribute);
				PoFileConfig.RegisterFileActions(PoFileActions);
			}

			PoFileDataSource.Reset(NewObject<UContentBrowserFileDataSource>(GetTransientPackage(), "PortableObjectData"));
			PoFileDataSource->Initialize(PoFileConfig);

			// Register the current paths that may contain localization data
			{
				TArray<FString> RootPaths;
				FPackageName::QueryRootContentPaths(RootPaths);
				for (const FString& RootPath : RootPaths)
				{
					OnContentPathMounted(RootPath, FPackageName::LongPackageNameToFilename(RootPath));
				}
			}
			
			// Listen for new paths that may contain localization data
			FPackageName::OnContentPathMounted().AddRaw(this, &FPortableObjectFileDataSourceModule::OnContentPathMounted);
			FPackageName::OnContentPathDismounted().AddRaw(this, &FPortableObjectFileDataSourceModule::OnContentPathDismounted);
			LocalizationDelegates::OnLocalizationTargetDataUpdated.AddRaw(this, &FPortableObjectFileDataSourceModule::OnLocalizationTargetDataUpdated);
		}
	}

	virtual void ShutdownModule() override
	{
		FPackageName::OnContentPathMounted().RemoveAll(this);
		FPackageName::OnContentPathDismounted().RemoveAll(this);
		LocalizationDelegates::OnLocalizationTargetDataUpdated.RemoveAll(this);
			
		PoFileDataSource.Reset();
	}

	virtual FDelegateHandle RegisterCanEditFileOverride(FCanEditFileDelegate&& Delegate) override
	{
		FDelegateHandle Handle(FDelegateHandle::GenerateNewHandle);
		CanEditFileOverrides.Add(MakeTuple(Handle, MoveTemp(Delegate)));
		return Handle;
	}

	virtual void UnregisterCanEditFileOverride(const FDelegateHandle& Handle) override
	{
		const int32 DelegateIndex = CanEditFileOverrides.IndexOfByPredicate([&Handle](const TTuple<FDelegateHandle, FCanEditFileDelegate>& DelegatePair)
		{
			return DelegatePair.Key == Handle;
		});

		if (DelegateIndex != INDEX_NONE)
		{
			CanEditFileOverrides.RemoveAt(DelegateIndex, EAllowShrinking::No);
		}
	}

private:
	void OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath)
	{
		if (PoFileDataSource)
		{
			const FString LocalizationTargetFolder = InFilesystemPath / TEXT("Localization");
			const FString VirtualMountPath = InAssetPath / TEXT("Localization");
			if (FPaths::DirectoryExists(LocalizationTargetFolder) && !PoFileDataSource->HasFileMount(*VirtualMountPath))
			{
				PoFileDataSource->AddFileMount(*VirtualMountPath, LocalizationTargetFolder);
			}
		}
	}

	void OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath)
	{
		if (PoFileDataSource)
		{
			PoFileDataSource->RemoveFileMount(*(InAssetPath / TEXT("Localization")));
		}
	}

	void OnLocalizationTargetDataUpdated(const FString& InLocalizationTargetPath)
	{
		if (PoFileDataSource)
		{
			// Trim the target name from the path so that we're querying for the outer "Localization" folder that hosts the target(s)
			const FString LocalizationTargetFolder = FPaths::GetPath(InLocalizationTargetPath);

			FString LocalizationTargetAssetPath;
			if (FPaths::DirectoryExists(LocalizationTargetFolder) && FPackageName::TryConvertFilenameToLongPackageName(LocalizationTargetFolder, LocalizationTargetAssetPath))
			{
				// Updating this target may have added a "Localization" folder that we're not currently monitoring
				// If so, add that mount now (any existing mounts will be updated via the directory watcher)
				if (!PoFileDataSource->HasFileMount(*LocalizationTargetAssetPath))
				{
					PoFileDataSource->AddFileMount(*LocalizationTargetAssetPath, LocalizationTargetFolder);
				}
			}
		}
	}

	bool CanEditFile(const FName InFilePath, const FString& InFilename, FText* OutErrorMsg)
	{
		for (auto DelegateIt = CanEditFileOverrides.CreateIterator(); DelegateIt; ++DelegateIt)
		{
			if (!DelegateIt->Value.IsBound())
			{
				DelegateIt.RemoveCurrent();
				continue;
			}

			if (!DelegateIt->Value.Execute(InFilePath, InFilename, OutErrorMsg))
			{
				return false;
			}
		}

		return true;
	}

	TStrongObjectPtr<UContentBrowserFileDataSource> PoFileDataSource;

	TArray<TTuple<FDelegateHandle, FCanEditFileDelegate>> CanEditFileOverrides;
};

IMPLEMENT_MODULE(FPortableObjectFileDataSourceModule, PortableObjectFileDataSource);

#undef LOCTEXT_NAMESPACE
