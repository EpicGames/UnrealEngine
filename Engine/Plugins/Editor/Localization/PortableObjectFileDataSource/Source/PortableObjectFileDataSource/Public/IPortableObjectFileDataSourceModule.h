// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleManager.h"

class IPortableObjectFileDataSourceModule : public IModuleInterface
{
public:
	static IPortableObjectFileDataSourceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPortableObjectFileDataSourceModule>("PortableObjectFileDataSource");
	}
	
	static IPortableObjectFileDataSourceModule* GetPtr()
	{
		return FModuleManager::GetModulePtr<IPortableObjectFileDataSourceModule>("PortableObjectFileDataSource");
	}

	using FCanEditFileDelegate = TDelegate<bool(const FName InFilePath, const FString& InFilename, FText* OutErrorMsg)>;

	/**
	 * Register an override handler for the "CanEdit" logic for Portable Object files.
	 * Each handler is queried in turn to see if they all allow editing of the file, and editing is refused if any return false.
	 */
	virtual FDelegateHandle RegisterCanEditFileOverride(FCanEditFileDelegate&& Delegate) = 0;
	
	/**
	 * Unregister an override handler for the "CanEdit" logic for Portable Object files, as previously registered by RegisterCanEditFileOverride.
	 */
	virtual void UnregisterCanEditFileOverride(const FDelegateHandle& Handle) = 0;
};
